{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NiagaraSimCaching",
	"Description": "Adds support for recording and playing back Niagara simulations in sequencer via take recorder",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/tutorials/Rk9v/unreal-engine-niagara-simulation-caching-in-sequencer",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraSimCaching",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "NiagaraSimCachingEditor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}