// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NiagaraSimCaching : ModuleRules
	{
        public NiagaraSimCaching(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
                    "EngineSettings",
                    "RenderCore",
                    "RHI",
					"Niagara",
					"MovieScene",
					"MovieSceneTracks",
					"LevelSequence", 
				});

			if(Target.bBuildEditor)
            {
				// Slate/Editor extensions
				PublicDependencyModuleNames.AddRange(
					new[]
					{
						"UnrealEd",
						"SlateCore",
						"Slate",
						"TakeRecorder"
					});
            }
		}
	}
}
