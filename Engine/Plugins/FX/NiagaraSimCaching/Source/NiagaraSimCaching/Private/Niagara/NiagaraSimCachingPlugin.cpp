// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/NiagaraSimCachingPlugin.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(INiagaraSimCachingPlugin, NiagaraSimCaching)

DEFINE_LOG_CATEGORY(LogNiagaraSimCaching)

void INiagaraSimCachingPlugin::StartupModule()
{
}

void INiagaraSimCachingPlugin::ShutdownModule() 
{
}
