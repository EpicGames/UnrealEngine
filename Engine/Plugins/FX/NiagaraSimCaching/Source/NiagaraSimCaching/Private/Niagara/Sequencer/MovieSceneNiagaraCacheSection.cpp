// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraCacheSection)

FMovieSceneNiagaraCacheParams::FMovieSceneNiagaraCacheParams() : FMovieSceneBaseCacheParams()
{
	SimCache = nullptr;
}

float FMovieSceneNiagaraCacheParams::GetSequenceLength() const
{
	if (SimCache)
	{
		return SimCache->GetDurationSeconds();
	}
	return 0.f;
}

UMovieSceneNiagaraCacheSection::UMovieSceneNiagaraCacheSection()
{
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
}

#if WITH_EDITOR
void UMovieSceneNiagaraCacheSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// When invoked without a specific changed property (e.g. Python set_editor_properties),
	// we don't know which member of Params was touched, so conservatively fire both branches.
	const bool bMatchesParamsMember = !PropertyChangedEvent.MemberProperty
		|| PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieSceneNiagaraCacheSection, Params);

	if (bMatchesParamsMember &&
		(!PropertyChangedEvent.Property || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMovieSceneNiagaraCacheParams, SimCache)))
	{
		if (TOptional<TRange<FFrameNumber>> AutoSizeRange = GetAutoSizeRange(); AutoSizeRange.IsSet())
		{
			SetRange(AutoSizeRange.GetValue());
		}
		bCacheOutOfDate = false;
	}

	if (bMatchesParamsMember &&
		(!PropertyChangedEvent.Property || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMovieSceneNiagaraCacheParams, CacheParameters)))
	{
		bCacheOutOfDate = true;
	}
}
#endif

UMovieSceneNiagaraCacheSection::UMovieSceneNiagaraCacheSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ParamsPtr = &Params;
#if WITH_EDITOR
	InitializePlayRate();
#endif
}




