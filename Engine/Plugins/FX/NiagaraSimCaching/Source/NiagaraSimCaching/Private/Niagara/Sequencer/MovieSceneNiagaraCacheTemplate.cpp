// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraCacheTemplate.h"

#include "IMovieScenePlayer.h"
#include "MovieSceneExecutionToken.h"
#include "NiagaraComponent.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/CoreMiscDefines.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraCacheTemplate)

DECLARE_CYCLE_STAT(TEXT("Niagara Cache - Token Execute"), MovieSceneEval_NiagaraCache_TokenExecute, STATGROUP_MovieSceneEval);

/** Used to set Manual Tick back to previous when outside section */
struct FPreAnimatedNiagaraCacheTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			explicit FToken(const UNiagaraComponent* InNiagaraComponent)
			{
				AgeMode = InNiagaraComponent->GetAgeUpdateMode();
			}

			virtual void RestoreState(UObject& ObjectToRestore, const UE::MovieScene::FRestoreStateParams& Params) override
			{
				UNiagaraComponent* NiagaraComponent = CastChecked<UNiagaraComponent>(&ObjectToRestore);
				NiagaraComponent->SetAgeUpdateMode(AgeMode);
				NiagaraComponent->SetSimCache(nullptr);
			}

			ENiagaraAgeUpdateMode AgeMode;
		};

		return FToken(CastChecked<UNiagaraComponent>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedNiagaraCacheTokenProducer>();
	}
};

/** A movie scene execution token that executes a NiagaraCache */
struct FNiagaraCacheExecutionToken : IMovieSceneExecutionToken
{
	FNiagaraCacheExecutionToken(const TOptional<FMovieSceneNiagaraSectionTemplateParameter>& InSection, const ENiagaraSimCacheSectionPlayMode& TargetReplayMode) :
		CacheSection(InSection), ReplayMode(TargetReplayMode)
	{}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_NiagaraCache_TokenExecute)

		UNiagaraComponent* NiagaraComponent = nullptr;
		if (Operand.ObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(Operand))
			{
				NiagaraComponent = Cast<UNiagaraComponent>(WeakObj);
				if (NiagaraComponent)
				{
					break;
				}
			}
		}
		if (!NiagaraComponent)
		{
			return;
		}

		if (!CacheSection.IsSet())
		{
			// outside of cached sections, use configured replay mode
			if (ReplayMode == ENiagaraSimCacheSectionPlayMode::DisplayCacheOnly)
			{
				NiagaraComponent->DeactivateImmediate();
			}
			else if (ReplayMode == ENiagaraSimCacheSectionPlayMode::SimWithoutCache)
			{
				NiagaraComponent->SetSimCache(nullptr);
			}
			return;
		}
		
		FMovieSceneNiagaraSectionTemplateParameter& Section = CacheSection.GetValue();
		if (NiagaraComponent->GetSimCache() != Section.Params.SimCache)
		{
			NiagaraComponent->SetSimCache(Section.Params.SimCache);
		}

		// calculate the time at which to evaluate the sim cache
		float Age = MapTimeToAnimation(Section.Params, Section.Params.SimCache->GetDurationSeconds(), Context.GetTime(), Context.GetFrameRate(), Section) + Section.Params.SimCache->GetStartSeconds();

		FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, PersistentData.GetSectionKey(), true);
		Player.SavePreAnimatedState(*NiagaraComponent, FPreAnimatedNiagaraCacheTokenProducer::GetAnimTypeID(), FPreAnimatedNiagaraCacheTokenProducer());
		NiagaraComponent->SetDesiredAge(Age);
		NiagaraComponent->SetAgeUpdateMode(ENiagaraAgeUpdateMode::DesiredAge);
		if (!NiagaraComponent->IsActive())
		{
			NiagaraComponent->ResetSystem();
		}
	}

	float MapTimeToAnimation(const FMovieSceneBaseCacheParams& BaseParams, float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution, const FMovieSceneNiagaraSectionTemplateParameter& Section) const
	{
		FFrameNumber SectionStartTime = Section.GetInclusiveStartFrame();
		FFrameNumber SectionEndTime = Section.GetExclusiveEndFrame();
		
		const float SequenceLength = Section.Params.SectionStretchMode == ENiagaraSimCacheSectionStretchMode::TimeDilate ? static_cast<float>(InTickResolution.AsSeconds(SectionEndTime - SectionStartTime)) : ComponentDuration;
		const FFrameTime AnimationLength = SequenceLength * InTickResolution;
		const int32 LengthInFrames = AnimationLength.FrameNumber.Value + static_cast<int>(AnimationLength.GetSubFrame() + 0.5f) + 1;
	
		//we only play end if we are not looping, and assuming we are looping if Length is greater than default length;
		const bool bLooping = (SectionEndTime.Value - SectionStartTime.Value + BaseParams.StartFrameOffset + BaseParams.EndFrameOffset) > LengthInFrames;

		InPosition = FMath::Clamp(InPosition, FFrameTime(SectionStartTime), FFrameTime(SectionEndTime - 1));

		const float SectionPlayRate = BaseParams.PlayRate;
		const float AnimPlayRate = FMath::IsNearlyZero(SectionPlayRate) ? 1.0f : SectionPlayRate;
		const float SeqLength = SequenceLength - static_cast<float>(InTickResolution.AsSeconds(BaseParams.StartFrameOffset + BaseParams.EndFrameOffset));

		float AnimPosition = static_cast<float>(FFrameTime::FromDecimal((InPosition - SectionStartTime).AsDecimal() * AnimPlayRate) / InTickResolution);
		AnimPosition += static_cast<float>(InTickResolution.AsSeconds(BaseParams.FirstLoopStartFrameOffset));
		if (SeqLength > 0.f && (bLooping || !FMath::IsNearlyEqual(AnimPosition, SeqLength, 1e-4f)))
		{
			AnimPosition = FMath::Fmod(AnimPosition, SeqLength);
		}
		AnimPosition += static_cast<float>(InTickResolution.AsSeconds(BaseParams.StartFrameOffset));
		if (BaseParams.bReverse)
		{
			AnimPosition = SequenceLength - AnimPosition;
		}
		if (Section.Params.SectionStretchMode == ENiagaraSimCacheSectionStretchMode::TimeDilate)
		{
			AnimPosition = (AnimPosition / SequenceLength) * ComponentDuration;
		}

		return AnimPosition;
	}

	TOptional<FMovieSceneNiagaraSectionTemplateParameter> CacheSection;
	ENiagaraSimCacheSectionPlayMode ReplayMode;
};

struct FNiagaraCacheResetToken : IMovieSceneExecutionToken
{
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_NiagaraCache_TokenExecute)

		UNiagaraComponent* NiagaraComponent = nullptr;
		if (Operand.ObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(Operand))
			{
				NiagaraComponent = Cast<UNiagaraComponent>(WeakObj);
				if (NiagaraComponent)
				{
					break;
				}
			}
		}
		if (!NiagaraComponent)
		{
			return;
		}

		NiagaraComponent->SetSimCache(nullptr);
	}
};

FMovieSceneNiagaraCacheSectionTemplate::FMovieSceneNiagaraCacheSectionTemplate(TArray<FMovieSceneNiagaraSectionTemplateParameter> InCacheSections)
	: CacheSections(InCacheSections)
{
}

void FMovieSceneNiagaraCacheSectionTemplate::Evaluate(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData,	FMovieSceneExecutionTokens& ExecutionTokens) const
{
	UMovieSceneNiagaraCacheTrack* NiagaraTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track.GetSourceTrack());
	if (NiagaraTrack == nullptr || NiagaraTrack->bIsRecording)
	{
		return;
	}
	ExecutionTokens.SetContext(Context);

	ENiagaraSimCacheSectionPlayMode TargetReplayMode = ENiagaraSimCacheSectionPlayMode::DisplayCacheOnly;
	TOptional<FMovieSceneNiagaraSectionTemplateParameter> CacheSectionToApply;
	bool bHasValidSections = false;
	for (const FMovieSceneNiagaraSectionTemplateParameter& Section : CacheSections)
	{
		bHasValidSections |= Section.Params.SimCache && Section.Params.SimCache->IsCacheValid();
		TargetReplayMode = Section.Params.CacheReplayPlayMode;
		if (Section.IsTimeWithinSection(Context.GetTime().GetFrame()) && Section.Params.SimCache)
		{
			CacheSectionToApply = Section;
			break;
		}
	}

	// remove the sim cache if we don't have section data to display anywhere
	if (!bHasValidSections)
	{
		ExecutionTokens.Add(FNiagaraCacheResetToken());
		return;
	}
	
	// only add a token if there isn't one already, otherwise another track's token takes precendence
	FMovieSceneSharedDataId TokenID = UMovieSceneNiagaraSystemTrack::SharedDataId;
	FNiagaraSharedMarkerToken* SharedCacheToken = static_cast<FNiagaraSharedMarkerToken*>(ExecutionTokens.FindShared(TokenID));
	if (SharedCacheToken == nullptr)
	{
		ExecutionTokens.Add(FNiagaraCacheExecutionToken(CacheSectionToApply, TargetReplayMode));
		FNiagaraSharedMarkerToken SharedToken = FNiagaraSharedMarkerToken();
		SharedToken.BoundObjectIDs.Add(Operand.ObjectBindingID);
		ExecutionTokens.AddShared(TokenID, SharedToken);
	}
	else if (!SharedCacheToken->BoundObjectIDs.Contains(Operand.ObjectBindingID))
	{
		ExecutionTokens.Add(FNiagaraCacheExecutionToken(CacheSectionToApply, TargetReplayMode));
		SharedCacheToken->BoundObjectIDs.Add(Operand.ObjectBindingID);
	}
}
