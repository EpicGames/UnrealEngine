// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneTrackImplementation.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"
#include "MovieSceneNiagaraCacheTemplate.generated.h"

USTRUCT()
struct FMovieSceneNiagaraSectionTemplateParameter
{
	GENERATED_BODY()
	
	UPROPERTY()
	FMovieSceneFrameRange SectionRange;

	UPROPERTY()
	FMovieSceneNiagaraCacheParams Params;
	
	bool IsTimeWithinSection(const FFrameNumber& Position) const 
	{
		return SectionRange.Value.Contains(Position);
	}

	FFrameNumber GetInclusiveStartFrame() const
	{
		TRangeBound<FFrameNumber> LowerBound = SectionRange.GetLowerBound();
		return LowerBound.IsInclusive() ? LowerBound.GetValue() : LowerBound.GetValue() + 1;
	}

	FFrameNumber GetExclusiveEndFrame() const
	{
		TRangeBound<FFrameNumber> UpperBound = SectionRange.GetUpperBound();
		return UpperBound.IsInclusive() ? UpperBound.GetValue() + 1 : UpperBound.GetValue();
	}
};

USTRUCT()
struct FMovieSceneNiagaraCacheSectionTemplate : public FMovieSceneTrackImplementation
{
	GENERATED_BODY()

	FMovieSceneNiagaraCacheSectionTemplate() {}
	FMovieSceneNiagaraCacheSectionTemplate(TArray<FMovieSceneNiagaraSectionTemplateParameter> CacheSections);

	virtual void SetupOverrides() override
	{
		EnableOverrides(CustomEvaluateFlag);
	}

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;


	UPROPERTY()
	TArray<FMovieSceneNiagaraSectionTemplateParameter> CacheSections;
};
