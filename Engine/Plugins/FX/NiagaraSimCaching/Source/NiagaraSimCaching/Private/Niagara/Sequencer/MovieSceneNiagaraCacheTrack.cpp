// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/Sequencer/MovieSceneNiagaraCacheTrack.h"
#include "MovieScene.h"
#include "MovieSceneNiagaraCacheTemplate.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"
#include "NiagaraSimCache.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraCacheTrack)

#define LOCTEXT_NAMESPACE "MovieSceneNiagaraCacheTrack"

/* UMovieSceneNiagaraCacheTrack constructor
 *****************************************************************************/

UMovieSceneNiagaraCacheTrack::UMovieSceneNiagaraCacheTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(19, 58, 171, 80);
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
	EvalOptions.bEvaluateInPreroll = true;
}

/* UMovieSceneNiagaraCacheTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneNiagaraCacheTrack::AddNewAnimation(FFrameNumber KeyTime, UNiagaraComponent* NiagaraComponent)
{
	UMovieSceneNiagaraCacheSection* NewSection = Cast<UMovieSceneNiagaraCacheSection>(CreateNewSection());
	{
		//const FFrameTime AnimationLength = NiagaraCache->CacheCollection->GetMaxDuration() * GetTypedOuter<UMovieScene>()->GetTickResolution();
		//const int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;
		//NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, INDEX_NONE);
	}

	AddSection(*NewSection);

	return NewSection;
}

TArray<UMovieSceneSection*> UMovieSceneNiagaraCacheTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (auto Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraCacheTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneEvalTemplatePtr();
}

/* UMovieSceneTrack interface
 *****************************************************************************/

const TArray<UMovieSceneSection*>& UMovieSceneNiagaraCacheTrack::GetAllSections() const
{
	return AnimationSections;
}

bool UMovieSceneNiagaraCacheTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneNiagaraCacheSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraCacheTrack::CreateNewSection()
{
	UMovieSceneNiagaraCacheSection* MovieSceneNiagaraCacheSection = NewObject<UMovieSceneNiagaraCacheSection>(this, NAME_None, RF_Transactional);
	MovieSceneNiagaraCacheSection->Params.SimCache = NewObject<UNiagaraSimCache>(MovieSceneNiagaraCacheSection, NAME_None, RF_Transactional);
	MovieSceneNiagaraCacheSection->Params.CacheParameters.AttributeCaptureMode = ENiagaraSimCacheAttributeCaptureMode::RenderingOnly;
	MovieSceneNiagaraCacheSection->Params.CacheParameters.bAllowInterpolation = true;
	MovieSceneNiagaraCacheSection->Params.CacheParameters.bAllowVelocityExtrapolation = true;
	return MovieSceneNiagaraCacheSection;
}

bool UMovieSceneNiagaraCacheTrack::PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
{
	UMovieSceneNiagaraCacheTrack* This = const_cast<UMovieSceneNiagaraCacheTrack*>(this);
	// We always want to evaluate everything
	OutData.Add(TRange<FFrameNumber>::All(), FMovieSceneTrackEvaluationData::FromTrack(This));

	return true;
}

void UMovieSceneNiagaraCacheTrack::PostCompile(FMovieSceneEvaluationTrack& OutTrack, const FMovieSceneTrackCompilerArgs& Args) const
{
	TArray<FMovieSceneNiagaraSectionTemplateParameter> TemplateParams;
	for (UMovieSceneSection* Section : AnimationSections)
	{
		if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
		{
			if (NiagaraCacheSection->IsActive() && NiagaraCacheSection->Params.SimCache)
			{
				FMovieSceneNiagaraSectionTemplateParameter& Param = TemplateParams.AddDefaulted_GetRef();
				Param.SectionRange = NiagaraCacheSection->SectionRange;
				Param.Params = NiagaraCacheSection->Params;
			}
		}
	}
	
	OutTrack.SetTrackImplementation(FMovieSceneNiagaraCacheSectionTemplate(TemplateParams));
	OutTrack.SetEvaluationPriority(10000);
}

void UMovieSceneNiagaraCacheTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();
}

bool UMovieSceneNiagaraCacheTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}

void UMovieSceneNiagaraCacheTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}

void UMovieSceneNiagaraCacheTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneNiagaraCacheTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}

bool UMovieSceneNiagaraCacheTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneNiagaraCacheTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Niagara Sim Cache");
}

#endif

void UMovieSceneNiagaraCacheTrack::ResetCache()
{
	for (TObjectPtr<UMovieSceneSection>& Section : AnimationSections)
	{
		if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
		{
			if (UNiagaraSimCache* NiagaraSimCache = NiagaraCacheSection->Params.SimCache)
			{
				//TODO (mga) not sure if we need to do anything here, as the cache is cleared when BeginWrite is called on it
				//NiagaraSimCache->ResetCache();
			}
		}
	}
}

void UMovieSceneNiagaraCacheTrack::SetCacheRecordingAllowed(bool bShouldRecord)
{
	bCacheRecordingEnabled = bShouldRecord;
}

bool UMovieSceneNiagaraCacheTrack::IsCacheRecordingAllowed() const
{
	bool bAllSectionCanRecord = true;
	for (UMovieSceneSection* Section : AnimationSections)
	{
		if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
		{
			if (NiagaraCacheSection->IsLocked() || NiagaraCacheSection->Params.bLockCacheToReadOnly)
			{
				bAllSectionCanRecord = false;
			}
		}
	}
	return bCacheRecordingEnabled && !IsEvalDisabled() && bAllSectionCanRecord;
}

int32 UMovieSceneNiagaraCacheTrack::GetMinimumEngineScalabilitySetting() const
{
	int32 MinSetting = -1;
	for (UMovieSceneSection* Section : AnimationSections)
	{
		if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
		{
			if (NiagaraCacheSection->Params.bOverrideQualityLevel)
			{
				MinSetting = FMath::Max(MinSetting, static_cast<int32>(NiagaraCacheSection->Params.RecordQualityLevel));
			}
		}
	}
	return MinSetting;
}

#undef LOCTEXT_NAMESPACE
