// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraSimCache.h"
#include "Sections/MovieSceneBaseCacheSection.h"
#include "PerQualityLevelProperties.h"
#include "MovieSceneNiagaraCacheSection.generated.h"

#define UE_API NIAGARASIMCACHING_API

UENUM(BlueprintType)
enum class ENiagaraSimCacheSectionPlayMode : uint8
{
	/**
	When the sequence has no cached data to display, the Niagara component runs the simulation normally
	*/
	SimWithoutCache,

	/**
	When the sequence has no cached data to display, the Niagara component is disabled
	*/
	DisplayCacheOnly,
};

UENUM(BlueprintType)
enum class ENiagaraSimCacheSectionStretchMode : uint8
{
	/**
	When the cache section is stretched in the track it will repeat the cached data 
	*/
	Repeat,

	/**
	When the cache section is stretched in the track it will dilate the input time so the cached data is stretched once over the full section
	*/
	TimeDilate,
};

USTRUCT()
struct FMovieSceneNiagaraCacheParams : public FMovieSceneBaseCacheParams
{
	GENERATED_BODY()

	UE_API FMovieSceneNiagaraCacheParams();
	virtual ~FMovieSceneNiagaraCacheParams() override = default;

	/** Gets the animation sequence length, not modified by play rate */
	UE_API virtual float GetSequenceLength() const override;

	UPROPERTY(EditAnywhere, Category = "NiagaraCache")
	FNiagaraSimCacheCreateParameters CacheParameters;
	
	/** The sim cache this section plays and records into */
	UPROPERTY(EditAnywhere, Category = "NiagaraCache")
	TObjectPtr<UNiagaraSimCache> SimCache;

	/** If true then the section properties might still be changed (so the section itself is not locked), but the cache cannot be rerecorded to prevent accidentally overriding the data within */
	UPROPERTY(EditAnywhere, Category = "NiagaraCache")
	bool bLockCacheToReadOnly = false;

	UPROPERTY(EditAnywhere, Category = "NiagaraCache", meta=(InlineEditConditionToggle))
	bool bOverrideQualityLevel = false;

	/** If set, then the engine scalability setting will be overriden with this value when recording a new cache for this track */
	UPROPERTY(EditAnywhere, Category = "NiagaraCache", meta=(EditCondition="bOverrideQualityLevel"))
	EPerQualityLevels RecordQualityLevel = EPerQualityLevels::Cinematic;

	/** What should the effect do when the track has no cache data to display */
	UPROPERTY(EditAnywhere, Category="SimCache")
	ENiagaraSimCacheSectionPlayMode CacheReplayPlayMode = ENiagaraSimCacheSectionPlayMode::DisplayCacheOnly;

	/** What should the effect do when the cache section is stretched? */
	UPROPERTY(EditAnywhere, Category="SimCache")
	ENiagaraSimCacheSectionStretchMode SectionStretchMode = ENiagaraSimCacheSectionStretchMode::TimeDilate;

#if WITH_EDITORONLY_DATA 
	/** True if the cache should be recorded at a rate that is slower than the sequencer play rate. */
	UPROPERTY(EditAnywhere, Category = "Cache", meta = (InlineEditConditionToggle))
	bool bOverrideRecordRate = false;

	/** The rate at which the cache should be recorded. Will be ignored if the sequence plays at a lower rate than this number.
	 * This option is useful if you need to play the sequence at a very high fps for simulation stability (e.g. 240fps), but then later want to play back the cache at a normal rate (e.g. 24fps).
	 * Please note that using this option might result in rendering artifacts for things like motion blur, velocity interpolation or inconsistent grid data in fluids, due to the dropped cache frames. 
	 */
	UPROPERTY(EditAnywhere, Category = "Cache", meta = (EditCondition = "bOverrideRecordRate"))
	float CacheRecordRateFPS = 24.0f;
#endif
};

/**
 * Movie scene section that control NiagaraCache playback
 */
UCLASS(MinimalAPI)
class UMovieSceneNiagaraCacheSection : public UMovieSceneBaseCacheSection
{
	GENERATED_UCLASS_BODY()

public:

	UMovieSceneNiagaraCacheSection();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY(EditAnywhere, Category = "NiagaraCache", meta = (ShowOnlyInnerProperties))
	FMovieSceneNiagaraCacheParams Params;

	UPROPERTY()
	bool bCacheOutOfDate = false;
};

#undef UE_API
