// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Tracks/MovieSceneCachedTrack.h"
#include "MovieSceneNiagaraCacheTrack.generated.h"

/**
 * Handles animation of NiagaraCache
 */
UCLASS(MinimalAPI)
class UMovieSceneNiagaraCacheTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
	, public IMovieSceneCachedTrack
{
	GENERATED_UCLASS_BODY()

public:

	/** Adds a new animation to this track */
	virtual UMovieSceneSection* AddNewAnimation(FFrameNumber KeyTime, class UNiagaraComponent* NiagaraComponent);

	/** Gets the animation sections at a certain time */
	TArray<UMovieSceneSection*> GetAnimSectionsAtTime(FFrameNumber Time);

	// UMovieSceneTrack interface
	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const override;
	virtual void PostCompile(FMovieSceneEvaluationTrack& OutTrack, const FMovieSceneTrackCompilerArgs& Args) const override;

	// ~IMovieSceneTrackTemplateProducer interface
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

	// ~IMovieSceneCachedTrack interface
	virtual void ResetCache() override;
	virtual void SetCacheRecordingAllowed(bool bShouldRecord) override;
	virtual bool IsCacheRecordingAllowed() const override;
	virtual int32 GetMinimumEngineScalabilitySetting() const override;
	
	UPROPERTY(Transient)
	bool bIsRecording = false;

private:

	/** List of all animation sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> AnimationSections;

	UPROPERTY()
	bool bCacheRecordingEnabled = true;
};
