// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NiagaraSimCachingEditor : ModuleRules
	{
        public NiagaraSimCachingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new[]
				{
					"Core",
					"CoreUObject",
					"Slate",
					"SlateCore",
					"InputCore",
					"Engine",
					"UnrealEd",
					"PropertyEditor",
					"ToolMenus",
					"Niagara",
					"NiagaraSimCaching",
					"LevelSequence",
                    "MovieScene",
                    "MovieSceneTools",
                    "MovieSceneTracks",
                    "SequencerCore",
                    "Sequencer",
                    "TimeManagement",
					"EditorFramework",
					"TakesCore",
					"TakeRecorder",
					"TakeTrackRecorders",
					"CacheTrackRecorder",
				});
		}
	}
}
