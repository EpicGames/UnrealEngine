// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/NiagaraSimCachingEditorPlugin.h"
#include "Features/IModularFeatures.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISequencerModule.h"
#include "Sequencer/NiagaraCacheTrackEditor.h"

IMPLEMENT_MODULE(INiagaraSimCachingEditorPlugin, NiagaraSimCachingEditor)

DEFINE_LOG_CATEGORY(LogNiagaraSimCachingEditor);

#define LOCTEXT_NAMESPACE "CacheEditorPlugin"

static const FName MovieSceneTrackRecorderFactoryName("MovieSceneTrackRecorderFactory");

void INiagaraSimCachingEditorPlugin::StartupModule()
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorBindingHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FNiagaraCacheTrackEditor::CreateTrackEditor));
		
	IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneNiagaraCacheTrackRecorder);
}

void INiagaraSimCachingEditorPlugin::ShutdownModule()
{
	if(UObjectInitialized())
	{
		if (ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
		{
			SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
		}

		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneNiagaraCacheTrackRecorder);
	}
}

#undef LOCTEXT_NAMESPACE
