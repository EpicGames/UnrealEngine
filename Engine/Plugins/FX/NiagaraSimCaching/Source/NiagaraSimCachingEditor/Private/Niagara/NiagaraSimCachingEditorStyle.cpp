// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCachingEditorStyle.h"

#include "Styling/SlateStyleRegistry.h"

FName FNiagaraSimCachingEditorStyle::StyleName("NiagaraSimCachingEditorStyle");

FNiagaraSimCachingEditorStyle::FNiagaraSimCachingEditorStyle() : FSlateStyleSet(StyleName)
{
	FVector2D Icon16x16(16.0f, 16.0f);
	
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("FX/NiagaraSimCaching/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("Niagara.SimCaching.StatusIcon.Color", FLinearColor(0, 1.f, 1.f));
	Set("Niagara.SimCaching.RecordIconSmall", new FSlateImageBrush(RootToContentDir(TEXT("RecordButton_Idle.png")), Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FNiagaraSimCachingEditorStyle::~FNiagaraSimCachingEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FNiagaraSimCachingEditorStyle& FNiagaraSimCachingEditorStyle::Get()
{
	static FNiagaraSimCachingEditorStyle Inst;
	return Inst;
}


