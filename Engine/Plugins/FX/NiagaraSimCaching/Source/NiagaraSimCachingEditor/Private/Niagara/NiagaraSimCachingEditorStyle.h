// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Slate style set that defines all the styles for the take recorder UI
 */
class FNiagaraSimCachingEditorStyle
	: public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FNiagaraSimCachingEditorStyle& Get();

private:

	FNiagaraSimCachingEditorStyle();
	virtual ~FNiagaraSimCachingEditorStyle() override;
};
