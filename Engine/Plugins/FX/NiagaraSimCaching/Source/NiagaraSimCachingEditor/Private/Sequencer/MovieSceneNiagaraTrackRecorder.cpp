// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneNiagaraTrackRecorder.h"

#include "ISequencer.h"
#include "LevelSequence.h"
#include "NiagaraComponent.h"
#include "TakeRecorderSettings.h"
#include "Misc/App.h"
#include "Misc/CoreMiscDefines.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "Niagara/NiagaraSimCachingEditorPlugin.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheTrack.h"
#include "Recorder/TakeRecorderParameters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraTrackRecorder)

#define LOCTEXT_NAMESPACE "MovieSceneNiagaraTrackRecorder"

bool FMovieSceneNiagaraTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<UNiagaraComponent>();
}

UMovieSceneTrackRecorder* FMovieSceneNiagaraTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneNiagaraTrackRecorder>();
}

UMovieSceneTrackRecorder* FMovieSceneNiagaraTrackRecorderFactory::CreateTrackRecorderForCacheTrack(IMovieSceneCachedTrack* CachedTrack, const TObjectPtr<ULevelSequence>& Sequence, const TSharedPtr<ISequencer>& Sequencer) const
{
	if (UMovieSceneNiagaraCacheTrack* NiagaraCacheTrack = Cast<UMovieSceneNiagaraCacheTrack>(CachedTrack))
	{
		UMovieSceneNiagaraTrackRecorder* TrackRecorder = NewObject<UMovieSceneNiagaraTrackRecorder>();
		
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		const TArray<FMovieSceneBinding>& SceneBindings = ((const UMovieScene*)MovieScene)->GetBindings();

		for (const FMovieSceneBinding& Binding : SceneBindings)
		{
			TArray<UMovieSceneTrack*> ComponentTracks = Binding.GetTracks();
			// find the Niagara component the track is bound to
			if (ComponentTracks.Contains(NiagaraCacheTrack))
			{
				FGuid ObjectGuid = Binding.GetObjectGuid();
				TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectGuid, Sequencer->GetFocusedTemplateID());
				for (auto& Bound : BoundObjects)
				{
					if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(Bound))
					{
						TrackRecorder->SystemToRecord = NiagaraComponent;
						NiagaraComponent->SetSimCache(nullptr);
						break;
					}
				}
				TrackRecorder->NiagaraCacheTrack = NiagaraCacheTrack;
				TrackRecorder->ObjectGuid = ObjectGuid;
				TrackRecorder->OwningTakeRecorderSource = nullptr;
				TrackRecorder->ObjectToRecord = TrackRecorder->SystemToRecord;
				TrackRecorder->MovieScene = MovieScene;
				TrackRecorder->Settings = nullptr;

				for (UMovieSceneTrack* Track : ComponentTracks)
				{
					if (UMovieSceneNiagaraSystemTrack* SystemTrack = Cast<UMovieSceneNiagaraSystemTrack>(Track))
					{
						if (SystemTrack->IsEvalDisabled() == false)
						{
							for (UMovieSceneSection* Section : SystemTrack->GetAllSections())
							{
								TRange<FFrameNumber> SectionRange = Section->GetRange();
								// we move the end of the section one frame out because cache interpolation needs a last frame to work correctly (otherwise the last frame could only be extrapolated)
								SectionRange.SetUpperBoundValue(SectionRange.GetUpperBoundValue().Value + 1);
								if (!TrackRecorder->RecordRange.IsSet())
								{
									TrackRecorder->RecordRange = SectionRange;
								}
								TrackRecorder->RecordRange = FFrameNumberRange::Hull(*TrackRecorder->RecordRange, SectionRange);
							}
						}
					}
				}

				TArray<UMovieSceneSection*> SceneSections = NiagaraCacheTrack->GetAllSections();
				if (SceneSections.Num() > 0)
				{
					TrackRecorder->NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(SceneSections[0]);
				}
				else
				{
					TrackRecorder->NiagaraCacheSection = CastChecked<UMovieSceneNiagaraCacheSection>(NiagaraCacheTrack->CreateNewSection());
					TrackRecorder->NiagaraCacheSection->SetIsActive(false);
					NiagaraCacheTrack->AddSection(*TrackRecorder->NiagaraCacheSection);
				}
				NiagaraCacheTrack->bIsRecording = true;

				return TrackRecorder;
			}
		}
	}
	return nullptr;
}

UMovieSceneTrackRecorder* FMovieSceneNiagaraTrackRecorderFactory::CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const
{
	return nullptr;
}

void UMovieSceneNiagaraTrackRecorder::CreateTrackImpl()
{
	SystemToRecord = CastChecked<UNiagaraComponent>(ObjectToRecord.Get());

	NiagaraCacheTrack = MovieScene->FindTrack<UMovieSceneNiagaraCacheTrack>(ObjectGuid);
	if (!NiagaraCacheTrack.IsValid())
	{
		NiagaraCacheTrack = MovieScene->AddTrack<UMovieSceneNiagaraCacheTrack>(ObjectGuid);
	}
	else
	{
		NiagaraCacheTrack->RemoveAllAnimationData();
	}

	if (NiagaraCacheTrack.IsValid())
	{
		NiagaraCacheTrack->bIsRecording = true;
		NiagaraCacheSection = CastChecked<UMovieSceneNiagaraCacheSection>(NiagaraCacheTrack->CreateNewSection());
		NiagaraCacheSection->SetIsActive(false);
		NiagaraCacheTrack->AddSection(*NiagaraCacheSection);

		// Resize the section to either it's remaining	keyframes range or 0
		NiagaraCacheSection->SetRange(NiagaraCacheSection->GetAutoSizeRange().Get(TRange<FFrameNumber>(0, 0)));

		// Make sure it starts at frame 0, in case Auto Size removed a piece of the start
		NiagaraCacheSection->ExpandToFrame(0);
	}
}

bool UMovieSceneNiagaraTrackRecorder::ShouldContinueRecording(const FQualifiedFrameTime& FrameTime) const
{
	if (RecordRange.IsSet())
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameNumber CurrentFrame = FrameTime.ConvertTo(TickResolution).FloorToFrame();
		return CurrentFrame <= RecordRange.GetValue().GetUpperBoundValue();
	}
	return true;
}

void UMovieSceneNiagaraTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
{
	if (NiagaraCacheSection.IsValid() && NiagaraCacheTrack.IsValid())
	{
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		FTakeRecorderParameters Parameters;
		Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

		if (RecordRange.IsSet())
		{
			NiagaraCacheSection->TimecodeSource = FMovieSceneTimecodeSource(FTimecode::FromFrameNumber(RecordRange->GetLowerBoundValue(), TickResolution));
			NiagaraCacheSection->SetRange(RecordRange.GetValue());
			NiagaraCacheSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(RecordRange->GetLowerBoundValue()));
		} 
		else
		{
			NiagaraCacheSection->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);
		}

		if (SystemToRecord.IsValid())
		{
			// start simulation and writing to the sim cache 
			SystemToRecord->SetSimCache(nullptr);
			//SystemToRecord->SetAgeUpdateMode(ENiagaraAgeUpdateMode::TickDeltaTime);
			//SystemToRecord->ResetSystem();
			if (NiagaraCacheSection->Params.SimCache == nullptr)
			{
				NiagaraCacheSection->Params.SimCache = NewObject<UNiagaraSimCache>(NiagaraCacheSection.Get(), NAME_None, RF_Transactional);
			}
			NiagaraCacheSection->Params.SimCache->BeginWrite(NiagaraCacheSection->Params.CacheParameters, SystemToRecord.Get());

			bRecordedFirstFrame = false;
			bRecordingEnabled = false;
			bRequestFinalize = false;
			PostEditorTickHandle = GEngine->OnPostEditorTick().AddUObject(this, &UMovieSceneNiagaraTrackRecorder::OnRecordFrame);
			NiagaraCacheSection->bCacheOutOfDate = false;
		}
	}
}

UMovieSceneSection* UMovieSceneNiagaraTrackRecorder::GetMovieSceneSection() const
{
	return Cast<UMovieSceneSection>(NiagaraCacheSection.Get());
}

void UMovieSceneNiagaraTrackRecorder::FinalizeTrackImpl()
{
	bRequestFinalize = true;
}

void UMovieSceneNiagaraTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime)
{
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	RecordingFrameNumber = CurrentFrameTime.ConvertTo(TickResolution).FloorToFrame();

	bool bRecordCurrentFrame = true;
	if (NiagaraCacheSection.IsValid() && NiagaraCacheSection->Params.bOverrideRecordRate)
	{
		if (LastRecordedFrame.Value > 0 && NiagaraCacheSection->Params.CacheRecordRateFPS != 0)
		{
			double ElapsedTime = TickResolution.AsInterval() * (RecordingFrameNumber.Value - LastRecordedFrame.Value);
			double DesiredTime = 1.0 / NiagaraCacheSection->Params.CacheRecordRateFPS;
			if (ElapsedTime < DesiredTime && !FMath::IsNearlyEqual(ElapsedTime,DesiredTime))
			{
				bRecordCurrentFrame = false;
			}
		}
	}

	bRecordingEnabled = bRecordCurrentFrame && (!RecordRange.IsSet() || (RecordingFrameNumber >= RecordRange->GetLowerBoundValue() && RecordingFrameNumber < RecordRange->GetUpperBoundValue()));
}

void UMovieSceneNiagaraTrackRecorder::OnRecordFrame(float DeltaSeconds)
{
	if (bRecordingEnabled && NiagaraCacheSection.IsValid() && SystemToRecord.IsValid())
	{
		FNiagaraSimCacheFeedbackContext FeedbackContext;
		FeedbackContext.bAutoLogIssues = false;
		if (NiagaraCacheSection->Params.SimCache->WriteFrame(SystemToRecord.Get(), FeedbackContext))
		{
			if (!bRecordedFirstFrame)
			{
				// set to the actual first recorded frame, because systems with spawn rate can tick for a few frames without having particles
				NiagaraCacheSection->SetStartFrame(RecordingFrameNumber);
				bRecordedFirstFrame = true;
			}

			// Expand the section to the new length
			NiagaraCacheSection->SetEndFrame(RecordingFrameNumber);
			
			LastRecordedFrame = RecordingFrameNumber;
		}

		for (const FString& Warning : FeedbackContext.Warnings)
		{
			UE_LOGF(LogNiagaraSimCachingEditor, Warning, "Recording sim cache for frame %i: %ls", RecordingFrameNumber.Value, *Warning);
		}

		for (const FString& Error : FeedbackContext.Errors)
		{
			UE_LOGF(LogNiagaraSimCachingEditor, Warning, "Unable to record sim cache for frame %i: %ls", RecordingFrameNumber.Value, *Error);
		}
	}

	if (bRequestFinalize)
	{
		GEngine->OnPostEditorTick().Remove(PostEditorTickHandle);
		PostEditorTickHandle.Reset();

		if (NiagaraCacheTrack.IsValid())
		{
			NiagaraCacheTrack->bIsRecording = false;
		}

		if (NiagaraCacheSection.IsValid())
		{
			// Finalize the sim cache
			NiagaraCacheSection->Params.SimCache->EndWrite(true);

			// Activate the section
			NiagaraCacheSection->SetIsActive(true);
		}
	}
}

#undef LOCTEXT_NAMESPACE
