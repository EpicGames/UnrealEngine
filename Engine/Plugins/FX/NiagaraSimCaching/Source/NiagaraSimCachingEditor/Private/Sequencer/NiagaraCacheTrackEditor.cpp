// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/NiagaraCacheTrackEditor.h"

#include "AssetToolsModule.h"
#include "CoreMinimal.h"
#include "Editor.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelSequence.h"
#include "Niagara/NiagaraSimCachingEditorStyle.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheTrack.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "NiagaraComponent.h"
#include "NiagaraSimCache.h"
#include "NiagaraSystem.h"
#include "Recorder/CacheTrackRecorder.h"
#include "SequencerSectionPainter.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TimeToPixel.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"

namespace NiagaraCacheEditorConstants
{
	constexpr float AnimationTrackHeight = 28.f;
}

#define LOCTEXT_NAMESPACE "FNiagaraCacheTrackEditor"

static UNiagaraComponent* AcquireNiagaraComponentFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;
	return Cast<UNiagaraComponent>(BoundObject);
}

FNiagaraCacheSection::FNiagaraCacheSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneNiagaraCacheSection>(&InSection))
	, Sequencer(InSequencer)
	, InitialFirstLoopStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{ }

UMovieSceneSection* FNiagaraCacheSection::GetSectionObject()
{
	return &Section;
}

FText FNiagaraCacheSection::GetSectionTitle() const
{
	if (Section.Params.SimCache != nullptr)
	{
		FNumberFormattingOptions FormatOptions;
		FormatOptions.MaximumFractionalDigits = 1;
		return FText::Format(FText::FromString("Sim Cache ({0} frames/{1}s)"), Section.Params.SimCache->GetNumFrames(), FText::AsNumber(Section.Params.SimCache->GetDurationSeconds(), &FormatOptions));
	
	}
	return LOCTEXT("NoNiagaraCacheSection", "No NiagaraCache");
}

float FNiagaraCacheSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(NiagaraCacheEditorConstants::AnimationTrackHeight);
}

int32 FNiagaraCacheSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	using namespace UE::Sequencer;

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Add lines where the animation starts and ends/loops
	const float AnimPlayRate = FMath::IsNearlyZero(Section.Params.PlayRate) ? 1.0f : Section.Params.PlayRate;
	const float Duration = Section.Params.GetSequenceLength();
	const float SeqLength = Duration - TickResolution.AsSeconds(Section.Params.StartFrameOffset + Section.Params.EndFrameOffset) / AnimPlayRate;
	const float FirstLoopSeqLength = SeqLength - TickResolution.AsSeconds(Section.Params.FirstLoopStartFrameOffset) / AnimPlayRate;

	if (!FMath::IsNearlyZero(SeqLength, KINDA_SMALL_NUMBER) && SeqLength > 0 && Section.Params.SectionStretchMode == ENiagaraSimCacheSectionStretchMode::Repeat)
	{
		float MaxOffset = Section.GetRange().Size<FFrameTime>() / TickResolution;
		float OffsetTime = FirstLoopSeqLength;
		float StartTime = Section.GetInclusiveStartFrame() / TickResolution;

		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixelConverter.SecondsToPixel(StartTime + OffsetTime) - TimeToPixelConverter.SecondsToPixel(StartTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += SeqLength;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (Painter.bIsSelected && SequencerPtr.IsValid())
	{
		FFrameTime CurrentTime = SequencerPtr->GetLocalTime().Time;
		if (Section.GetRange().Contains(CurrentTime.FrameNumber) && Section.Params.SimCache != nullptr)
		{
			const float Time = TimeToPixelConverter.FrameToPixel(CurrentTime);

			UNiagaraSimCache* NiagaraCache = Section.Params.SimCache;
			int32 NumFrames = NiagaraCache ? NiagaraCache->GetNumFrames() : 0;

			// Draw the current time next to the scrub handle
			const float AnimTime = Section.MapTimeToAnimation(Duration, CurrentTime, TickResolution);
			int32 FrameTime = Duration ? FMath::FloorToFloat((AnimTime / Duration) * NumFrames) + 1 : 0;
			if (Section.Params.SectionStretchMode == ENiagaraSimCacheSectionStretchMode::TimeDilate && Duration)
			{
				float Frames = (Section.GetExclusiveEndFrame() - Section.GetInclusiveStartFrame()).Value;
				float NormalizedAge = (CurrentTime.FrameNumber - Section.GetInclusiveStartFrame()).Value / Frames;
				FrameTime = FMath::FloorToFloat(NormalizedAge * NumFrames) + 1;
			}
			FString FrameString =  FString::FromInt(FrameTime);

			const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static constexpr float TextOffsetPx = 10.f;
			bool  bDrawLeft = (Painter.SectionGeometry.Size.X - Time) < (TextSize.X + 22.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? Time - TextSize.X - TextOffsetPx : Time + TextOffsetPx;
			//handle mirrored labels
			constexpr float MajorTickHeight = 7.0f;
			FVector2D TextOffset(TextPosition, Painter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

			const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
			const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
			// draw time string

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId + 100,
				Painter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);

			FSlateDrawElement::MakeText(
				Painter.DrawElements,
				LayerId + 101,
				Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				DrawEffects,
				DrawColor
			);

		}
	}

	return LayerId;
}

void FNiagaraCacheSection::BeginResizeSection()
{
	InitialFirstLoopStartOffsetDuringResize = Section.Params.FirstLoopStartFrameOffset;
	InitialStartTimeDuringResize = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : 0;
}

void FNiagaraCacheSection::UpdateSection(FFrameNumber& UpdateTime) const
{
	const FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber StartOffset = FrameRate.AsFrameNumber((UpdateTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

	StartOffset += InitialFirstLoopStartOffsetDuringResize;

	if (StartOffset < 0)
	{
		// Ensure start offset is not less than 0 and adjust ResizeTime
		UpdateTime = UpdateTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}
	else
	{
		// If the start offset exceeds the length of one loop, trim it back.
		const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
		StartOffset = StartOffset % SeqLength;
	}

	Section.Params.FirstLoopStartFrameOffset = StartOffset;
}

void FNiagaraCacheSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	// Adjust the start offset when resizing from the beginning
	if (ResizeMode == SSRM_LeadingEdge)
	{
		UpdateSection(ResizeTime);
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FNiagaraCacheSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FNiagaraCacheSection::SlipSection(FFrameNumber SlipTime)
{
	UpdateSection(SlipTime);

	ISequencerSection::SlipSection(SlipTime);
}

void FNiagaraCacheSection::BeginDilateSection()
{
	Section.PreviousPlayRate = Section.Params.PlayRate; //make sure to cache the play rate
}
void FNiagaraCacheSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	Section.Params.PlayRate = Section.PreviousPlayRate / DilationFactor;
	Section.SetRange(NewRange);
}

FNiagaraCacheTrackEditor::FNiagaraCacheTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }

TSharedRef<ISequencerTrackEditor> FNiagaraCacheTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FNiagaraCacheTrackEditor(InSequencer));
}

bool FNiagaraCacheTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	if (InSequence && InSequence->IsTrackSupported(UMovieSceneNiagaraCacheTrack::StaticClass()) == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence && InSequence->IsA(ULevelSequence::StaticClass());
}

bool FNiagaraCacheTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneNiagaraCacheTrack::StaticClass();
}

TSharedRef<ISequencerSection> FNiagaraCacheTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FNiagaraCacheSection(SectionObject, GetSequencer()));
}

FText FNiagaraCacheTrackEditor::GetDisplayName() const
{
	return LOCTEXT("NiagaraCacheTrackEditor_DisplayName", "Niagara");
}

void FNiagaraCacheTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UNiagaraComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		if(ObjectBindings.Num() > 0)
		{
			if (AcquireNiagaraComponentFromObjectGuid(ObjectBindings[0], GetSequencer()))
			{
				UMovieSceneTrack* Track = nullptr;

				MenuBuilder.AddMenuEntry(
					NSLOCTEXT("Sequencer", "AddNiagaraCache", "Niagara Cache"),
					NSLOCTEXT("Sequencer", "AddNiagaraCacheTooltip", "Adds a Niagara Cache track."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FNiagaraCacheTrackEditor::BuildNiagaraCacheTrack, ObjectBindings, Track)
					)
				);
				
			}
		}
	}
}

void FNiagaraCacheTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	MenuBuilder.BeginSection("CacheActions", LOCTEXT("CacheActions", "Cache Actions"));
	{
		if (UMovieSceneNiagaraCacheTrack* CacheTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track))
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CreateAssetFromThisCache", "Save Cache To Asset"),
				LOCTEXT("CreateAssetFromThisCacheToolTip", "Create a simulation cache asset from this track's simulation data. This is especially useful if the cached data is very big or should be iterated on outside of sequencer.\nOnly enabled if there is valid cache data available and it's not already saved in an asset."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([CacheTrack]()
					{
						UMovieSceneNiagaraCacheSection* CacheSection = Cast<UMovieSceneNiagaraCacheSection>(CacheTrack->GetAllSections()[0]);
						UNiagaraSimCache* SimCacheToSave = CacheSection->Params.SimCache;

						const FString PackagePath = FPackageName::GetLongPackagePath(SimCacheToSave->GetOutermost()->GetName());
						const FName CacheName = FName(CacheTrack->GetDisplayName().ToString());
						
						FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

						// Duplicate and Save the sim cache.
						UNiagaraSimCache* CreatedAsset = Cast<UNiagaraSimCache>(AssetToolsModule.Get().DuplicateAssetWithDialogAndTitle(CacheName.ToString(), PackagePath, SimCacheToSave, LOCTEXT("CreateCacheAssetDialogTitle", "Save Sim Cache As")));
						if (CreatedAsset != nullptr)
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CreatedAsset);

							// Replace existing cache with asset
							FScopedTransaction ScopedTransaction(LOCTEXT("CreateAssetFromCache", "Create asset from cache"));
							CacheSection->Modify();
							CacheSection->Params.SimCache = CreatedAsset;
						}
					}),
					FCanExecuteAction::CreateLambda(
						[CacheTrack]()
						{
							TArray<UMovieSceneSection*> Sections = CacheTrack->GetAllSections();
							if (Sections.Num() == 1)
							{
								if (UMovieSceneNiagaraCacheSection* CacheSection = Cast<UMovieSceneNiagaraCacheSection>(Sections[0]))
								{
									return CacheSection->Params.SimCache && !CacheSection->Params.SimCache->IsEmpty() && !CacheSection->Params.SimCache->IsAsset();
								}
							}
							return false;
						}
					)
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("RecordCache", "Record cache"),
				LOCTEXT("RecordCacheToolTip", "Refresh the data in the selected cache tracks by playing the sequence and recording the Niagara system."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]()
					{
						UCacheTrackRecorder::RecordSelectedTracks(GetSequencer());
					}),
					FCanExecuteAction::CreateUObject(CacheTrack, &UMovieSceneNiagaraCacheTrack::IsCacheRecordingAllowed)
				)
			);
		}
	}

	MenuBuilder.EndSection();
}

void FNiagaraCacheTrackEditor::BuildNiagaraCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track)
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SequencerPtr.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNiagaraCache_Transaction", "Add Niagara Cache"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			if (UNiagaraComponent* NiagaraComponent = AcquireNiagaraComponentFromObjectGuid(ObjectBinding, SequencerPtr))
				{
					AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FNiagaraCacheTrackEditor::AddKeyInternal, NiagaraComponent, Track));
				}
		}
	}
}

FKeyPropertyResult FNiagaraCacheTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UNiagaraComponent* NiagaraComponent, UMovieSceneTrack* Track)
{
	FKeyPropertyResult KeyPropertyResult;

	const FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(NiagaraComponent);
	const FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		if (!Track)
		{
			Track = AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectHandle, UMovieSceneNiagaraCacheTrack::StaticClass(), NAME_None);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(Track))
		{
			Track->Modify();

			UMovieSceneNiagaraCacheTrack* CacheTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track);
			UMovieSceneSection* NewSection = CacheTrack->AddNewAnimation(KeyTime, NiagaraComponent);
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			if (NiagaraComponent->GetAsset())
			{
				CacheTrack->SetDisplayName(FText::Format(LOCTEXT("NiagaraCacheTrack_Name", "{0} Sim Cache"), FText::FromName(NiagaraComponent->GetAsset()->GetFName())));
			}

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}

FReply FNiagaraCacheTrackEditor::RecordCacheTrack(IMovieSceneCachedTrack* Track)
{
	UCacheTrackRecorder::RecordCacheTrack(Track, GetSequencer());
	return FReply::Handled();
}

bool FNiagaraCacheTrackEditor::IsCacheTrackOutOfDate(UMovieSceneTrack* Track)
{
	if (UMovieSceneNiagaraCacheTrack* NiagaraTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track))
	{
		for (UMovieSceneSection* Section : NiagaraTrack->GetAllSections())
		{
			if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
			{
				if (NiagaraCacheSection->bCacheOutOfDate)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FNiagaraCacheTrackEditor::HasConflictingLifecycleTrack(UMovieSceneTrack* CacheTrack) const
{
	const UMovieScene* MovieScene = GetMovieSceneSequence()->GetMovieScene();
	FFrameNumber PlaybackStartFrame = MovieScene->GetPlaybackRange().GetLowerBoundValue();
	const TArray<FMovieSceneBinding>& SceneBindings = MovieScene->GetBindings();

	for (const FMovieSceneBinding& Binding : SceneBindings)
	{
		TArray<UMovieSceneTrack*> ComponentTracks = Binding.GetTracks();
		// find any life cycle tracks bound to the same component
		if (ComponentTracks.Contains(CacheTrack))
		{
			for (UMovieSceneTrack* Track : ComponentTracks)
			{
				if (UMovieSceneNiagaraSystemTrack* SystemTrack = Cast<UMovieSceneNiagaraSystemTrack>(Track))
				{
					if (SystemTrack->IsEvalDisabled() == false)
					{
						for (UMovieSceneSection* Section : SystemTrack->GetAllSections())
						{
							// check if we start at the same frame as playback. If we do and don't use desired age then that's a problem, as recording a cache will reset the system multiple times at the start and invalidate the recording.
							UMovieSceneNiagaraSystemSpawnSection* SpawnSection = Cast<UMovieSceneNiagaraSystemSpawnSection>(Section);
							if (SpawnSection && SpawnSection->GetAgeUpdateMode() == ENiagaraAgeUpdateMode::TickDeltaTime && SpawnSection->GetInclusiveStartFrame() == PlaybackStartFrame)
							{
								return true;
							}
						}
					}
				}
			}
		}
	}
	return false;
}

FText FNiagaraCacheTrackEditor::GetCacheTrackWarnings(UMovieSceneTrack* Track) const
{
	TArray<FText> Warnings;
	if (HasConflictingLifecycleTrack(Track))
	{
		Warnings.Add(LOCTEXT("AddNiagaraCache_LifecycleTrackWarn", "The system life cycle track associated with this cache track is not set up well for cache recording.\nEither (1) set it to use desired age mode or (2) move it so it starts on a different frame than the sequence playback (e.g. one frame earlier)."));
	}
	if (IsCacheTrackOutOfDate(Track))
	{
		Warnings.Add(LOCTEXT("AddNiagaraCache_OutOfDate", "This cache track is out of date, as it's properties were changed after recording. Consider re-recording the cached data."));
	}
	return FText::Join(FText::FromString("\n\n"), Warnings);
}

TSharedPtr<SWidget> FNiagaraCacheTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		TViewModelPtr<IObjectBindingExtension> ObjectBindingModel = Params.ViewModel->FindAncestorOfType<IObjectBindingExtension>();
		FGuid ObjectBinding = ObjectBindingModel ? ObjectBindingModel->GetObjectGuid() : FGuid();

		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		FWeakObjectPtr BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding) : nullptr;
		

		UMovieSceneTrack* Track = nullptr;
		if (TViewModelPtr<ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<ITrackExtension>(true))
		{
			Track = TrackModel->GetTrack();
		}


		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(2, 0, 2, 0)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("NiagaraCacheTrack_RecordCache", "Refresh the data in this cache by playing the sequence and recording the Niagara system."))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ContentPadding(2)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsEnabled_Lambda([Track]()
			{
				IMovieSceneCachedTrack* CachedTrack = Cast<IMovieSceneCachedTrack>(Track);
				return CachedTrack && CachedTrack->IsCacheRecordingAllowed();
			})
			.OnClicked(this, &FNiagaraCacheTrackEditor::RecordCacheTrack, Cast<IMovieSceneCachedTrack>(Track))
			[
				SNew(SImage)
				.Image(FNiagaraSimCachingEditorStyle::Get().GetBrush( "Niagara.SimCaching.RecordIconSmall"))
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(2, 0, 2, 0)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.ToolTipText_Raw(this, &FNiagaraCacheTrackEditor::GetCacheTrackWarnings, Track)
			.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
			.Visibility_Lambda([Track, this]()
			{
				if (IsCacheTrackOutOfDate(Track) || HasConflictingLifecycleTrack(Track))
				{
					return EVisibility::Visible;
				}
				return EVisibility::Collapsed;
			})
		]
		+ SHorizontalBox::Slot()
		.Padding(2, 0, 2, 0)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.ToolTipText(LOCTEXT("AddNiagaraCache_InvalidCache", "The cache used by this track does not fit the Niagara component and will be ignored. Re-Record the sim cache or select a different one."))
			.Image(FAppStyle::Get().GetBrush("Icons.ErrorWithColor"))
			.Visibility_Lambda([Track, BoundObject]()
			{
				if (BoundObject.IsValid() && BoundObject.Get()->IsA(UNiagaraComponent::StaticClass()))
				{
					UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(BoundObject.Get());
					if (UMovieSceneNiagaraCacheTrack* NiagaraTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track))
					{
						for (UMovieSceneSection* Section : NiagaraTrack->GetAllSections())
						{
							if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
							{
								if (NiagaraCacheSection->Params.SimCache && !NiagaraCacheSection->Params.SimCache->IsEmpty() && !NiagaraCacheSection->Params.SimCache->CanRead(NiagaraComponent->GetAsset()))
								{
									return EVisibility::Visible;
								}
							}
						}
					}
				}
				return EVisibility::Collapsed;
			})
		]
		+ SHorizontalBox::Slot()
		.Padding(2, 0, 2, 0)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.ToolTipText(LOCTEXT("NiagaraCache_Active", "This track is currently providing cached data to the Niagara component, no simulation is running."))
			.Image(FSlateIconFinder::FindIconForClass(UNiagaraSimCache::StaticClass()).GetIcon())
			.ColorAndOpacity(FNiagaraSimCachingEditorStyle::Get().GetColor("Niagara.SimCaching.StatusIcon.Color"))
			.Visibility_Lambda([Track, BoundObject]()
			{
				if (BoundObject.IsValid() && BoundObject.Get()->IsA(UNiagaraComponent::StaticClass()))
				{
					UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(BoundObject.Get());
					if (UMovieSceneNiagaraCacheTrack* NiagaraTrack = Cast<UMovieSceneNiagaraCacheTrack>(Track))
					{
						for (UMovieSceneSection* Section : NiagaraTrack->GetAllSections())
						{
							if (UMovieSceneNiagaraCacheSection* NiagaraCacheSection = Cast<UMovieSceneNiagaraCacheSection>(Section))
							{
								if (!NiagaraTrack->IsEvalDisabled() && NiagaraCacheSection->Params.SimCache && NiagaraCacheSection->Params.SimCache->CanRead(NiagaraComponent->GetAsset()) && NiagaraComponent->GetSimCache() == NiagaraCacheSection->Params.SimCache && NiagaraComponent->IsActive())
								{
									return EVisibility::Visible;
								}
							}
						}
					}
				}
				return EVisibility::Collapsed;
			})
		];
	}

	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

const FSlateBrush* FNiagaraCacheTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(UNiagaraSimCache::StaticClass()).GetIcon();
}

#undef LOCTEXT_NAMESPACE
