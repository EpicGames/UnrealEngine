// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "CoreMinimal.h"
#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"

#include "MovieSceneNiagaraTrackRecorder.generated.h"

#define UE_API NIAGARASIMCACHINGEDITOR_API

class UNiagaraComponent;
class  UMovieSceneNiagaraCacheTrack;
class  UMovieSceneNiagaraCacheSection;

struct FFrameNumber;
class  UMovieSceneSection;
class  UMovieSceneTrackRecorderSettings;

class FMovieSceneNiagaraTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneNiagaraTrackRecorderFactory() = default;

	// ~Begin IMovieSceneTrackRecorderFactory Interface
	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;
	UE_API virtual class UMovieSceneTrackRecorder* CreateTrackRecorderForCacheTrack(IMovieSceneCachedTrack* CachedTrack, const TObjectPtr<ULevelSequence>& Sequence, const TSharedPtr<ISequencer>& Sequencer) const override;

	// Particle Systems are entire components and you can't animate them as a property
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override;
	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneNiagaraTrackRecorderFactory", "DisplayName", "Niagara Cache Track"); }
	// ~End IMovieSceneTrackRecorderFactory Interface
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneNiagaraTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()

public:
	// ~Begin UMovieSceneTrackRecorder Interface
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	UE_API virtual UMovieSceneSection* GetMovieSceneSection() const override;
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual bool ShouldContinueRecording(const FQualifiedFrameTime& FrameTime) const override;
	// ~End UMovieSceneTrackRecorder Interface

	/** Returns the Niagara cache track on which the cache manager will be recorded */
	TWeakObjectPtr<UMovieSceneNiagaraCacheTrack> GetNiagaraCacheTrack() const {return NiagaraCacheTrack;}

protected:
	UE_API void SetRecordingEnabled(bool bEnabled);
	UE_API void OnRecordFrame(float DeltaSeconds);
	
private:
	
	/** The NiagaraCache Track to record onto */
	TWeakObjectPtr<UMovieSceneNiagaraCacheTrack> NiagaraCacheTrack;

	/** Sections to record to on each track*/
	TWeakObjectPtr<UMovieSceneNiagaraCacheSection> NiagaraCacheSection;

	/** Object to record from */
	TWeakObjectPtr<UNiagaraComponent> SystemToRecord;

	TOptional<FFrameNumberRange> RecordRange;

	bool			bRecordedFirstFrame = false;
	bool			bRecordingEnabled = false;
	bool			bRequestFinalize = false;
	FFrameNumber	LastRecordedFrame;
	FFrameNumber	RecordingFrameNumber;
	FDelegateHandle	PostEditorTickHandle;

	friend FMovieSceneNiagaraTrackRecorderFactory;
};

#undef UE_API
