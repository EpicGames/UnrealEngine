// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class IMovieSceneCachedTrack;
/**
 * Editor for Niagara Cache tracks
 */
class FNiagaraCacheTrackEditor : public FMovieSceneTrackEditor
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FNiagaraCacheTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FNiagaraCacheTrackEditor() override = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ~Begin ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override; 
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	virtual const FSlateBrush* GetIconBrush() const override;
	// ~End ISequencerTrackEditor interface
	
private:

	/** Build an internal Niagara cache track */
	void BuildNiagaraCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, class UNiagaraComponent* NiagaraComponent, UMovieSceneTrack* Track);

	FReply RecordCacheTrack(IMovieSceneCachedTrack* Track);

	static bool IsCacheTrackOutOfDate(UMovieSceneTrack* Track);
	bool HasConflictingLifecycleTrack(UMovieSceneTrack* Track) const;
	FText GetCacheTrackWarnings(UMovieSceneTrack* Track) const;
};

/** Niagara cache sequencer section */
class FNiagaraCacheSection
	: public ISequencerSection
	, public TSharedFromThis<FNiagaraCacheSection>
{
public:

	/** Constructor. */
	FNiagaraCacheSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	// ~Begin ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	virtual void BeginSlipSection() override;
	virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual void BeginDilateSection() override;
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;
	// ~End ISequencerSection interface
	
private:

	void UpdateSection(FFrameNumber& UpdateTime) const;

	/** The section we are visualizing */
	class UMovieSceneNiagaraCacheSection& Section;

	/** Used to draw animation frame, need selection state and local time*/
	TWeakPtr<ISequencer> Sequencer;

	/** Cached first loop start offset value valid only during resize */
	FFrameNumber InitialFirstLoopStartOffsetDuringResize;
	
	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;
};
